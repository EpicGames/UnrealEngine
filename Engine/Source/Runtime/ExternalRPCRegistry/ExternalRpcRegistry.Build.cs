// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ExternalRpcRegistry : ModuleRules
{
    public ExternalRpcRegistry(ReadOnlyTargetRules Target) : base(Target)
    {
        PrivateDependencyModuleNames.AddRange(
            new string[] {
                "Core",
                "CoreUObject",
				"Json",
				"HTTP",
				"HTTPServer"
            }
        );
		PublicDefinitions.Add("USE_RPC_REGISTRY_IN_SHIPPING=0");
    }
}
