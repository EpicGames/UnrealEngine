// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExternalRpcRegistrationComponent.h"

#include "HttpModule.h"
#include "HttpServerResponse.h"
#include "ExternalRpcRegistry.h"

#include "Dom/JsonObject.h"
#include "Interfaces/IHttpRequest.h"
#include "Misc/CommandLine.h"
#include "Serialization/JsonSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ExternalRpcRegistrationComponent)


#ifndef WITH_RPC_REGISTRY
#define WITH_RPC_REGISTRY (USE_RPC_REGISTRY_IN_SHIPPING || !UE_BUILD_SHIPPING )
#endif

UExternalRpcRegistrationComponent::UExternalRpcRegistrationComponent()
{
	FParse::Value(FCommandLine::Get(), TEXT("externalrpclistenaddress="), ListenerAddress);
	FParse::Value(FCommandLine::Get(), TEXT("rpcsenderid="), SenderID);
}

TSharedPtr<FJsonObject> UExternalRpcRegistrationComponent::GetJsonObjectFromRequestBody(const TArray<uint8>& InRequestBody)
{
#if WITH_RPC_REGISTRY
	FUTF8ToTCHAR WByteBuffer(reinterpret_cast<const ANSICHAR*>(InRequestBody.GetData()), InRequestBody.Num());
	const FString IncomingRequestBody = FString::ConstructFromPtrSize(WByteBuffer.Get(), WByteBuffer.Length());
	TSharedPtr<FJsonObject> BodyObject = MakeShareable(new FJsonObject());
	TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(IncomingRequestBody);

	if (FJsonSerializer::Deserialize(JsonReader, BodyObject) && BodyObject.IsValid())
	{
		return BodyObject;
	}
#endif

	return nullptr;
}

void UExternalRpcRegistrationComponent::DeregisterHttpCallbacks()
{
#if WITH_RPC_REGISTRY
	for (const FName& RouteName : RegisteredRoutes)
	{
		UExternalRpcRegistry::GetInstance()->CleanUpRoute(RouteName);
	}
	RegisteredRoutes.Empty();
	BroadcastRpcListChanged();
#endif
}

void UExternalRpcRegistrationComponent::RegisterAlwaysOnHttpCallbacks()
{
#if WITH_RPC_REGISTRY
	BroadcastRpcListChanged();
#endif
}

bool UExternalRpcRegistrationComponent::HttpSendMessageToListener(FString MessageCategory, FString MessagePayload)
{
#if WITH_RPC_REGISTRY
	if (ListenerAddress == TEXT("") || SenderID == TEXT(""))
	{
		// We don't have a listener.
		return false;
	}

	FString RequestUri = FString::Printf(TEXT("http://%s/sendmessage"), *ListenerAddress);
	FHttpModule& HttpModule = FHttpModule::Get();

	FString RequestString;

	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> MessageRequest = HttpModule.CreateRequest();

	TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&RequestString);
	JsonWriter->WriteObjectStart();
	JsonWriter->WriteValue(TEXT("senderid"), SenderID);
	JsonWriter->WriteValue(TEXT("category"), MessageCategory);
	JsonWriter->WriteValue(TEXT("payload"), MessagePayload);
	JsonWriter->WriteObjectEnd();
	JsonWriter->Close();

	MessageRequest->SetVerb(TEXT("POST"));
	MessageRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
	MessageRequest->SetContentAsString(RequestString);
	MessageRequest->SetURL(RequestUri);

	bool bRequestProcessed = MessageRequest->ProcessRequest();

	if (bRequestProcessed)
	{
		GLog->Log(TEXT("BotRPC"), ELogVerbosity::Log, FString::Printf(TEXT("Successfully sent message to %s from SenderID %s! body: %s"), *MessageRequest->GetURL(), *SenderID, *RequestString));
	}
	else
	{
		GLog->Log(TEXT("BotRPC"), ELogVerbosity::Error, FString::Printf(TEXT("Failed to send message to %s from SenderID %s!"), *MessageRequest->GetURL(), *SenderID));
	}
	return bRequestProcessed;
#else
	return false;
#endif
}
void UExternalRpcRegistrationComponent::BroadcastRpcListChanged()
{
#if WITH_RPC_REGISTRY
	HttpSendMessageToListener(TEXT("RpcRegistry"), TEXT("RpcListUpdated"));
#endif
}
TUniquePtr<FHttpServerResponse> UExternalRpcRegistrationComponent::CreateSimpleResponse(bool bInWasSuccessful, FString InValue /*= ""*/, bool bInFatal /* = false*/)
{
	FString ResponseStr;
	TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&ResponseStr);
	JsonWriter->WriteObjectStart();
	JsonWriter->WriteValue(TEXT("succeeded"), bInWasSuccessful);
	JsonWriter->WriteValue(TEXT("value"), InValue);
	JsonWriter->WriteValue(TEXT("fatal"), bInFatal);
	JsonWriter->WriteObjectEnd();
	JsonWriter->Close();
	return FHttpServerResponse::Create(ResponseStr, TEXT("application/json"));
}

FHttpRequestHandler UExternalRpcRegistrationComponent::CreateRouteHandle(TDelegate<bool(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)> InFunc)
{
	FHttpRequestHandler OutHandler = nullptr;
#if WITH_RPC_REGISTRY
	OutHandler = FHttpRequestHandler::CreateLambda([this, InFunc](const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
		{
			// If we have a security token enabled, make sure we're honoring it.
			if (SecuritySecret != "")
			{
				if (!Request.Headers.Find(TEXT("authToken")))
				{
					TUniquePtr<FHttpServerResponse> Response = CreateSimpleResponse(false, TEXT("Missing authToken header"));
					Response->Code = EHttpServerResponseCodes::Forbidden;
					OnComplete(MoveTemp(Response));
					return true;
				}
				TArray<FString> ExistingHeader = Request.Headers[TEXT("authToken")];
				if (ExistingHeader[0] != SecuritySecret)
				{
					TUniquePtr<FHttpServerResponse> Response = CreateSimpleResponse(false, TEXT("Incorrect authToken header value"));
					Response->Code = EHttpServerResponseCodes::Forbidden;
					OnComplete(MoveTemp(Response));
					return true;
				}
			}

			UExternalRpcRegistry::GetInstance()->AddRequestToLedger(Request);

			FHttpResultCallback CompletionCallback;
			TSharedPtr<FJsonObject> RequestJson = GetJsonObjectFromRequestBody(Request.Body);
			if (RequestJson.IsValid() && RequestJson->HasField(TEXT("CaptureExecutionTime")) && RequestJson->GetBoolField(TEXT("CaptureExecutionTime")))
			{
				FRpcTimingContext TimingContext;
				TimingContext.StartTiming();

				CompletionCallback = [this, OnComplete, TimingContext](TUniquePtr<FHttpServerResponse> Response) mutable
				{
					TimingContext.StopTiming();
					// Inject timing into response JSON
					AppendTimingContextToResponse(Response, TimingContext);

					// Call original OnComplete to send response
					OnComplete(MoveTemp(Response));
				};
			}
			else
			{
				CompletionCallback = OnComplete;
			}

			return InFunc.Execute(Request, CompletionCallback);
		});
#endif
	return OutHandler;
}


void UExternalRpcRegistrationComponent::RegisterHttpCallback(FExternalRouteInfo InRouteInfo, const FHttpRequestHandler& Handler, bool bOverrideIfBound /* = false */)
{
#if WITH_RPC_REGISTRY
	const FName RouteName = InRouteInfo.RouteName;
	UExternalRpcRegistry::GetInstance()->RegisterNewRoute(MoveTemp(InRouteInfo), Handler, bOverrideIfBound);
	RegisteredRoutes.Add(RouteName);
#endif
}

void UExternalRpcRegistrationComponent::RegisterHttpCallback(FName RouteName, const FHttpPath& HttpPath, const EHttpServerRequestVerbs& RequestVerbs, const FHttpRequestHandler& Handler, bool bOverrideIfBound/* = false*/, FString OptionalCategory/* = TEXT("Unknown")*/, FString OptionalContentType/* = TEXT("")*/, TArray<FExternalRpcArgumentDesc> OptionalInArguments/* = TArray<FExternalRpcArgumentDesc>()*/)
{
#if WITH_RPC_REGISTRY
	FExternalRouteInfo NewRouteInfo;
	NewRouteInfo.RouteName = RouteName;
	NewRouteInfo.RoutePath = HttpPath;
	NewRouteInfo.RequestVerbs = RequestVerbs;
	NewRouteInfo.InputContentType = MoveTemp(OptionalContentType);
	NewRouteInfo.ExpectedArguments = MoveTemp(OptionalInArguments);
	NewRouteInfo.RpcCategory = OptionalCategory;
	RegisterHttpCallback(MoveTemp(NewRouteInfo), Handler, bOverrideIfBound);
#endif
}
bool UExternalRpcRegistrationComponent::HttpUpdateIpOnListener(FString TargetName, FString NewIP)
{
#if WITH_RPC_REGISTRY && !PLATFORM_WINDOWS
	if (TargetName == TEXT("") || NewIP == TEXT(""))
	{
		// We don't have a listener.
		GLog->Log(TEXT("BotRPC"), ELogVerbosity::Error, FString::Printf(TEXT("UpdateTargetIp Failed to send TargetName:  %s   NewIP: %s"), *TargetName, *NewIP));
		return false;
	}
	FString RequestUri = FString::Printf(TEXT("http://%s/updateip"), *ListenerAddress);
	FHttpModule& HttpModule = FHttpModule::Get();

	FString RequestString;

	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> MessageRequest = HttpModule.CreateRequest();

	TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&RequestString);
	JsonWriter->WriteObjectStart();
	JsonWriter->WriteValue(TEXT("target"), TargetName);
	JsonWriter->WriteValue(TEXT("newip"), NewIP);
	JsonWriter->WriteObjectEnd();
	JsonWriter->Close();

	MessageRequest->SetVerb(TEXT("POST"));
	MessageRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
	MessageRequest->SetContentAsString(RequestString);
	MessageRequest->SetURL(RequestUri);

	bool bRequestProcessed = MessageRequest->ProcessRequest();

	if (bRequestProcessed)
	{
		GLog->Log(TEXT("BotRPC"), ELogVerbosity::Log, FString::Printf(TEXT("UpdateTargetIp: Successfully sent message to %s from SenderID %s! body: %s"), *MessageRequest->GetURL(), *SenderID, *RequestString));
	}
	else
	{
		GLog->Log(TEXT("BotRPC"), ELogVerbosity::Error, FString::Printf(TEXT("UpdateTargetIp Failed to send message to %s from SenderID %s!"), *MessageRequest->GetURL(), *SenderID));
	}
	return bRequestProcessed;
#else
return false;
#endif
}

void UExternalRpcRegistrationComponent::AppendTimingContextToResponse(TUniquePtr<FHttpServerResponse>& InResponse, FRpcTimingContext& InTimingContext)
{
#if WITH_RPC_REGISTRY
	if (!InResponse.IsValid() || InResponse->Body.IsEmpty())
	{
		return;
	}

	// Only append timing context to a json response
	bool bIsJson = false;
	if (const TArray<FString>* ContentTypeHeaders = InResponse->Headers.Find(TEXT("Content-Type")))
	{
		for (const FString& ContentType : *ContentTypeHeaders)
		{
			if (ContentType.Contains(TEXT("application/json"), ESearchCase::IgnoreCase))
			{
				bIsJson = true;
				break;
			}
		}
	}
	if (!bIsJson)
	{
		return;
	}

	TSharedPtr<FJsonObject> ResponseJson = GetJsonObjectFromRequestBody(InResponse->Body);
	ResponseJson->SetNumberField(TEXT("ExecutionTimeMs"), InTimingContext.ExecutionTimeMs);

	// Re-serialize JSON
	FString ResponseStr;
	TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&ResponseStr);
	FJsonSerializer::Serialize(ResponseJson.ToSharedRef(), Writer);

	// Re-write the response body
	FTCHARToUTF8 ConvertToUtf8(*ResponseStr);
	const uint8* ConvertToUtf8Bytes = (reinterpret_cast<const uint8*>(ConvertToUtf8.Get()));
	InResponse->Body.Empty();
	InResponse->Body.Append(ConvertToUtf8Bytes, ConvertToUtf8.Length());
#endif
}
