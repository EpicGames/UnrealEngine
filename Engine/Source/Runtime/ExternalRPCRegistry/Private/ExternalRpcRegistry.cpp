// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExternalRpcRegistry.h"
#include "HttpServerModule.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Parse.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "HttpServerResponse.h"
#include "Serialization/JsonWriter.h"
#include "Logging/LogMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ExternalRpcRegistry)

#define USE_RPC_REGISTRY_IN_SHIPPING 0

#ifndef WITH_RPC_REGISTRY
#define WITH_RPC_REGISTRY (USE_RPC_REGISTRY_IN_SHIPPING || !UE_BUILD_SHIPPING )
#endif

DEFINE_LOG_CATEGORY(LogExternalRpcRegistry)

UExternalRpcRegistry* UExternalRpcRegistry::ObjectInstance = nullptr;


FString GetHttpRouteVerbString(EHttpServerRequestVerbs InVerbs)
{
#if WITH_RPC_REGISTRY
	switch (InVerbs)
	{
		case EHttpServerRequestVerbs::VERB_POST:
		{
			return TEXT("POST");
		}
		case EHttpServerRequestVerbs::VERB_PUT:
		{
			return TEXT("PUT");
		}
		case EHttpServerRequestVerbs::VERB_GET:
		{
			return TEXT("GET");
		}
		case EHttpServerRequestVerbs::VERB_PATCH:
		{
			return TEXT("PATCH");
		}
		case EHttpServerRequestVerbs::VERB_DELETE:
		{
			return TEXT("DELETE");
		}
		case EHttpServerRequestVerbs::VERB_NONE:
		{
			return TEXT("NONE");
		}
	}
#endif
	return TEXT("UNKNOWN");
}

bool UExternalRpcRegistry::IsActiveRpcCategory(FString InCategory)
{
#if WITH_RPC_REGISTRY
	if (!ActiveRpcCategories.Num() || ActiveRpcCategories.Contains(InCategory))
	{
		return true;
	}
#endif
	return false;
}

UExternalRpcRegistry::~UExternalRpcRegistry()
{
	CleanUpAllRoutes();
}


const FRpcConfig& UExternalRpcRegistry::GetRpcConfig()
{
	static FRpcConfig Config;
	static bool bHasInitialized = false;

	if (bHasInitialized)
	{
		return Config;
	}
	bHasInitialized = true;


	// look for multiple -rpcport or -rpcconfig commandline options
	int PortNum;
	for (const TCHAR* Cursor = FCommandLine::Get(); (Cursor != nullptr) && FParse::Value(Cursor, TEXT("-rpcport="), PortNum, &Cursor); )
	{
		Config.ActiveHostPorts.AddUnique(PortNum);
		// set one as Default, just in case something asks
		if (!Config.PortMapping.Contains(TEXT("Default")))
		{
			Config.PortMapping.Add(TEXT("Default"), PortNum);
		}
	}

	// look for a -rpcconfig=FooRpc string 
	FString ConfigName;
	for (const TCHAR* Cursor = FCommandLine::Get(); (Cursor != nullptr) && FParse::Value(Cursor, TEXT("-rpcconfig="), ConfigName, false, &Cursor); )
	{
		const FConfigSection* Section = GConfig->GetSection(*ConfigName, false, GEngineIni);
		if (Section)
		{
			FString SelectorType;
			// put all the entries into the mapping
			for (const TPair<FName, FConfigValue>& Pair : *Section)
			{
				// look for a special selector key
				if (Pair.Key == TEXT("__Selector"))
				{
					SelectorType = Pair.Value.GetValue();
					continue;
				}

				int Port = FCString::Atoi(*Pair.Value.GetValue());
				if (Port > 0)
				{
					Config.PortMapping.Add(Pair.Key.ToString(), Port);
				}
			}

			// now look for the active port (or look for a default entry if none given)
			FString ActiveName = "Default";

			if (SelectorType == TEXT("RuntimeMode"))
			{
				ActiveName = GIsEditor ? TEXT("Editor") :
					GIsServer ? TEXT("Server") : TEXT("Client");
			}
			else
			{
				// look for rpcname, or use Default if none
				FParse::Value(FCommandLine::Get(), TEXT("-rpcname="), ActiveName);
			}

			// look up the active port, or use 0 for missing port (0 isn't a valid port)
			PortNum = Config.PortMapping.FindRef(ActiveName);

			if (PortNum > 0)
			{
				// allow for multiple clients, so client nums >= 1 we append the num. client 0 is just called Client
				if (ActiveName == TEXT("Client"))
				{
					int ClientNum = 0;
					if (FParse::Value(FCommandLine::Get(), TEXT("-RpcClientNum="), ClientNum) && ClientNum >= 0 && ClientNum < 1000)
					{
						PortNum += ClientNum;
					}
				}

				Config.ActiveHostPorts.AddUnique(PortNum);
			}
		}
	}

	return Config;
}

bool UExternalRpcRegistry::IsEnabled()
{
#if WITH_RPC_REGISTRY
	if (GetRpcConfig().ActiveHostPorts.Num() > 0)
	{
		return true;
	}
#endif
	return false;
}

UExternalRpcRegistry* UExternalRpcRegistry::GetInstance()
{
#if WITH_RPC_REGISTRY
	if (ObjectInstance == nullptr)
	{		
		ObjectInstance = NewObject<UExternalRpcRegistry>();
		FString InCommandLineValue;
		if (FParse::Value(FCommandLine::Get(), TEXT("enabledrpccategories="), InCommandLineValue))
		{
			TArray<FString> Substrings;
			InCommandLineValue.ParseIntoArray(Substrings, TEXT(","));
			for (int i = 0; i < Substrings.Num(); i++)
			{
				if (!ObjectInstance->ActiveRpcCategories.Contains(Substrings[i]))
				{
					ObjectInstance->ActiveRpcCategories.Add(Substrings[i]);
				}
			}
		}
		
		// look for override from commandline (we want a value here even if UExternalRpcRegistry::IsEnabled() returns false, so the 
		// routes below, and in other code, don't trigger an assert on registration)
		ObjectInstance->PortsToUse = GetRpcConfig().ActiveHostPorts;
		if (ObjectInstance->PortsToUse.Num() == 0)
		{
			ObjectInstance->PortsToUse.Add(0);
		}

		FParse::Value(FCommandLine::Get(), TEXT("rpcledgersize="), ObjectInstance->RequestLedgerCapacity);

		// We always want the ListRegisteredRpcs route bound, no matter what.

		FHttpRequestHandler ListRoutesRequestHandler = FHttpRequestHandler::CreateUObject(ObjectInstance, &ThisClass::HttpListOpenRoutes);
		ObjectInstance->RegisterNewRoute(TEXT("ListRegisteredRpcs"), FHttpPath("/listrpcs"), EHttpServerRequestVerbs::VERB_GET,
			ListRoutesRequestHandler, true, true);

		FHttpRequestHandler PrintLedgerRequestHandler = FHttpRequestHandler::CreateUObject(ObjectInstance, &ThisClass::HttpPrintRequestLedger);
		// We always want the ListRegisteredRpcs route bound, no matter what.
		ObjectInstance->RegisterNewRoute(TEXT("GetRequestHistory"), FHttpPath("/requesthistory"), EHttpServerRequestVerbs::VERB_GET,
			PrintLedgerRequestHandler, true, true);

		FHttpRequestHandler ListOASv3RequestHandler = FHttpRequestHandler::CreateUObject(ObjectInstance, &ThisClass::HttpListOASv3JSONRoutes);
		// /swagger.json escaped as %2e
		ObjectInstance->RegisterNewRoute(TEXT("ListSwaggerJson"), FHttpPath("/swagger.json"), EHttpServerRequestVerbs::VERB_GET,
			ListOASv3RequestHandler, true, true);

		FHttpRequestHandler SwaggerUIHandler = FHttpRequestHandler::CreateUObject(ObjectInstance, &ThisClass::HttpSwaggerUI);
		ObjectInstance->RegisterNewRoute(TEXT("SwaggerUIHTML"), FHttpPath("/swagger/index.html"), EHttpServerRequestVerbs::VERB_GET,
			SwaggerUIHandler, true, true);

		FHttpServerModule::Get().StartAllListeners();

		ObjectInstance->AddToRoot();
	}
#endif
	return ObjectInstance;
}

bool UExternalRpcRegistry::GetRegisteredRoute(FName RouteName, FExternalRouteInfo& OutRouteInfo)
{
#if WITH_RPC_REGISTRY
	if (RegisteredRoutes.Find(RouteName))
	{
		OutRouteInfo.RouteName = RouteName;
		OutRouteInfo.RoutePath = RegisteredRoutes[RouteName].Handle->Path;
		OutRouteInfo.RequestVerbs = RegisteredRoutes[RouteName].Handle->Verbs;
		OutRouteInfo.InputContentType = RegisteredRoutes[RouteName].InputContentType;
		OutRouteInfo.ExpectedArguments = RegisteredRoutes[RouteName].ExpectedArguments;
		return true;
	}
#endif
	return false;
}

void UExternalRpcRegistry::RegisterNewRouteWithArguments(FName RouteName, const FHttpPath& HttpPath, const EHttpServerRequestVerbs& RequestVerbs, const FHttpRequestHandler& Handler, TArray<FExternalRpcArgumentDesc> InArguments, bool bOverrideIfBound /* = false */, bool bIsAlwaysOn /* = false */, FString OptionalCategory /* = FString("Unknown") */, FString OptionalContentType /* = TEXT("")*/)
{
#if WITH_RPC_REGISTRY
	if (!IsEnabled())
	{
		return;
	}
	FExternalRouteInfo InRouteInfo;
	InRouteInfo.RouteName = RouteName;
	InRouteInfo.RoutePath = HttpPath;
	InRouteInfo.RequestVerbs = RequestVerbs;
	InRouteInfo.InputContentType = OptionalContentType;
	InRouteInfo.ExpectedArguments = MoveTemp(InArguments);
	InRouteInfo.RpcCategory = OptionalCategory;
	InRouteInfo.bAlwaysOn = bIsAlwaysOn;
	RegisterNewRoute(MoveTemp(InRouteInfo), Handler, bOverrideIfBound);
#endif
}


void UExternalRpcRegistry::RegisterNewRoute(FName RouteName, const FHttpPath& HttpPath, const EHttpServerRequestVerbs& RequestVerbs, const FHttpRequestHandler& Handler, bool bOverrideIfBound /* = false */, bool bIsAlwaysOn /* = false */, FString OptionalCategory /* = FString("Unknown") */, FString OptionalContentType /* = TEXT("")*/, FString OptionalExpectedFormat /*= TEXT("")*/)
{
#if WITH_RPC_REGISTRY
	if (!IsEnabled())
	{
		return;
	}
	FExternalRouteInfo InRouteInfo;
	InRouteInfo.RouteName = RouteName;
	InRouteInfo.RoutePath = HttpPath;
	InRouteInfo.RequestVerbs = RequestVerbs;
	InRouteInfo.InputContentType = OptionalContentType;
	InRouteInfo.RpcCategory = OptionalCategory;
	InRouteInfo.bAlwaysOn = bIsAlwaysOn;
	RegisterNewRoute(MoveTemp(InRouteInfo), Handler, bOverrideIfBound);
#endif
}

void UExternalRpcRegistry::RegisterNewRoute(FExternalRouteInfo InRouteInfo, const FHttpRequestHandler& Handler, bool bOverrideIfBound /* = false */)
{
#if WITH_RPC_REGISTRY
	if (!IsEnabled())
	{
		return;
	}

	if (!InRouteInfo.bAlwaysOn && !IsActiveRpcCategory(InRouteInfo.RpcCategory))
	{
		return;
	}

	// create a route/route name for each port - if there are multiple ports, then their will be a "foo.portnum" named route for each port
	// we also track a mapping from 'foo.portnum' to 'foo' for one of the routes, so when /listrpcs is called, we can report 'foo' as the rpc
	// name that users of the api will use. 'foo.portnum' is not meant to be exposed externally
	for (int PortToUse : PortsToUse)
	{
		TSharedPtr<IHttpRouter> HttpRouter = FHttpServerModule::Get().GetHttpRouter(PortToUse);

		// for multiport listening, we need a route for each one
		FName RoutePortName = InRouteInfo.RouteName;
		if (PortsToUse.Num() > 1)
		{
			RoutePortName = *FString::Printf(TEXT("%s.%d"), *InRouteInfo.RouteName.ToString(), PortToUse);
		}

		if (RegisteredRoutes.Find(RoutePortName))
		{
			if (!bOverrideIfBound)
			{
				UE_LOGF(LogExternalRpcRegistry, Warning, "Route with friendly key %ls already exists at location %ls - skipping duplicate registration.", *RoutePortName.ToString(), *InRouteInfo.RoutePath.GetPath());
				return;
			}
			UE_LOGF(LogExternalRpcRegistry, Log, "Overwriting route at friendly key %ls - from %ls to %ls ", *RoutePortName.ToString(), *RegisteredRoutes[RoutePortName].Handle->Path, *InRouteInfo.RoutePath.GetPath());

			HttpRouter->UnbindRoute(RegisteredRoutes[RoutePortName].Handle);
		}

		// keep a mapping of original route name to one of the new route names, for ListRPCs
		MultiPortRouteMapping.Add(RoutePortName, InRouteInfo.RouteName);

		FExternalRouteDesc RouteDesc;
		RouteDesc.Handle = HttpRouter->BindRoute(InRouteInfo.RoutePath, InRouteInfo.RequestVerbs, Handler);
		RouteDesc.InputContentType = InRouteInfo.InputContentType;
		RouteDesc.ExpectedArguments = InRouteInfo.ExpectedArguments;
		RouteDesc.RpcCategory = InRouteInfo.RpcCategory;
		RegisteredRoutes.Add(RoutePortName, RouteDesc);
		UE_LOGF(LogExternalRpcRegistry, Log, "Route name %ls was bound!", *RoutePortName.ToString());
	}

#endif
}

void UExternalRpcRegistry::CleanUpAllRoutes()
{
#if WITH_RPC_REGISTRY
	TArray<FName> OutRouteKeys;
	RegisteredRoutes.GetKeys(OutRouteKeys);
	for (const FName& RouteKey : OutRouteKeys)
	{
		CleanUpRoute(RouteKey);
	}
#endif
}

void UExternalRpcRegistry::CleanUpRoute(FName RouteName, bool bFailIfUnbound /* = false */)
{
#if WITH_RPC_REGISTRY
	if (RegisteredRoutes.Find(RouteName))
	{
		// get the port - if there are multiple ports, use the port from the end of the RouteName, otherwise, use the only PortToUse
		int Port = 0;
		if (PortsToUse.Num() > 1)
		{
			FString RouteNameString = RouteName.ToString();
			int DotIndex;
			if (!RouteNameString.FindLastChar('.', DotIndex))
			{
				UE_LOGF(LogExternalRpcRegistry, Warning, "Expected a .portnum in the route name %ls when using multiport", *RouteNameString);
				check(!bFailIfUnbound);
				return;
			}
			Port = FCString::Atoi(*RouteNameString.Mid(DotIndex + 1));
		}
		else
		{
			Port = PortsToUse[0];
		}

		// remove the mapping
		MultiPortRouteMapping.Remove(RouteName);

		TSharedPtr<IHttpRouter> HttpRouter = FHttpServerModule::Get().GetHttpRouter(Port);
		HttpRouter->UnbindRoute(RegisteredRoutes[RouteName].Handle);
		RegisteredRoutes.Remove(RouteName);
		UE_LOGF(LogExternalRpcRegistry, Log, "Route name %ls was unbound!", *RouteName.ToString());
	}
	else
	{
		UE_LOGF(LogExternalRpcRegistry, Warning, "Route name %ls does not exist, could not unbind.", *RouteName.ToString());
		check(!bFailIfUnbound);
	}

#endif
}

bool UExternalRpcRegistry::HttpListOpenRoutes(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
#if WITH_RPC_REGISTRY
	FString ResponseStr;
	TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&ResponseStr);
	JsonWriter->WriteArrayStart();
	for (const TPair<FName, FName>& RouteKeyPair : MultiPortRouteMapping)
	{
		FName RouteKey = RouteKeyPair.Key;
		JsonWriter->WriteObjectStart();
		JsonWriter->WriteValue(TEXT("name"), RouteKeyPair.Value.ToString());
		JsonWriter->WriteValue(TEXT("route"), RegisteredRoutes[RouteKey].Handle->Path);
		JsonWriter->WriteValue(TEXT("verb"), GetHttpRouteVerbString(RegisteredRoutes[RouteKey].Handle->Verbs));
		if (!RegisteredRoutes[RouteKey].InputContentType.IsEmpty())
		{
			JsonWriter->WriteValue(TEXT("inputContentType"), RegisteredRoutes[RouteKey].InputContentType);
		}
		if (!RegisteredRoutes[RouteKey].ExpectedArguments.IsEmpty())
		{
			JsonWriter->WriteArrayStart(TEXT("args"));
			for (const FExternalRpcArgumentDesc& ArgDesc : RegisteredRoutes[RouteKey].ExpectedArguments)
			{
				JsonWriter->WriteObjectStart();
				JsonWriter->WriteValue(TEXT("name"), ArgDesc.Name);
				JsonWriter->WriteValue(TEXT("type"), ArgDesc.Type);
				JsonWriter->WriteValue(TEXT("desc"), ArgDesc.Desc);
				JsonWriter->WriteValue(TEXT("optional"), ArgDesc.bIsOptional);
				JsonWriter->WriteObjectEnd();
			}
			JsonWriter->WriteArrayEnd();
		}
		JsonWriter->WriteObjectEnd();
	}
	JsonWriter->WriteArrayEnd();
	JsonWriter->Close();
	auto Response = FHttpServerResponse::Create(ResponseStr, TEXT("application/json"));
	Response->Headers.Add(TEXT("Access-Control-Allow-Origin"), { TEXT("*") });
	OnComplete(MoveTemp(Response));
#endif
	return true;
}
void UExternalRpcRegistry::AddRequestToLedger(const FHttpServerRequest& Request)
{
#if WITH_RPC_REGISTRY
	if (Request.Headers.Find(TEXT("rpcname")))
	{
		FRpcLedgerEntry NewEntry;
		NewEntry.RpcName = Request.Headers[TEXT("rpcname")][0];
		FUTF8ToTCHAR WByteBuffer(reinterpret_cast<const ANSICHAR*>(Request.Body.GetData()), Request.Body.Num());
		NewEntry.RequestBody = FString::ConstructFromPtrSize(WByteBuffer.Get(), WByteBuffer.Length());
		NewEntry.RequestTime = FDateTime::UtcNow();
		RequestLedger.Add(NewEntry);
	}
	// Reduce ledger to proper max size.
	while (RequestLedger.Num() > RequestLedgerCapacity)
	{
		RequestLedger.RemoveAt(0);
	}
#endif
}

bool UExternalRpcRegistry::HttpPrintRequestLedger(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
#if WITH_RPC_REGISTRY
	FString ResponseStr;
	TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&ResponseStr);
	JsonWriter->WriteArrayStart();
	for (const FRpcLedgerEntry& LoggedRequest : RequestLedger)
	{
		JsonWriter->WriteObjectStart();
		JsonWriter->WriteValue(TEXT("rpcname"), LoggedRequest.RpcName);
		JsonWriter->WriteValue(TEXT("requesttimestamp"), LoggedRequest.RequestTime.ToString());
		JsonWriter->WriteValue(TEXT("requestbody"), LoggedRequest.RequestBody);

		JsonWriter->WriteObjectEnd();
	}
	JsonWriter->WriteArrayEnd();
	JsonWriter->Close();
	auto Response = FHttpServerResponse::Create(ResponseStr, TEXT("application/json"));
	Response->Headers.Add(TEXT("Access-Control-Allow-Origin"), { TEXT("*") });
	OnComplete(MoveTemp(Response));
#endif
	return true;
}

bool UExternalRpcRegistry::HttpSwaggerUI(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
#if WITH_RPC_REGISTRY
	/*TODO: Maybe read this from contents, but for now just bake it in code.*/
	FString ResponseStr = R"(<!DOCTYPE html>
<html lang="en">
<head>
  <meta charset="utf-8" />
  <meta name="viewport" content="width=device-width, initial-scale=1" />
  <meta
    name="description"
    content="SwaggerUI"
  />
  <title>SwaggerUI</title>
  <link rel="stylesheet" href="https://unpkg.com/swagger-ui-dist@4.4.1/swagger-ui.css" />
</head>
<body>
  <div id="swagger-ui"></div>
  <script src="https://unpkg.com/swagger-ui-dist@4.4.1/swagger-ui-bundle.js" crossorigin></script>
  <script src="https://unpkg.com/swagger-ui-dist@4.4.1/swagger-ui-standalone-preset.js" crossorigin></script>
  <script>
    window.onload = () => {
      window.ui = SwaggerUIBundle({
        url: 'http://{0}:{1}/swagger.json',
        dom_id: '#swagger-ui',
      });
    };
  </script>
</body>
</html>)";

	// Default to multihome address if provided, else use localhost.
	FString Address;
	if (FParse::Value(FCommandLine::Get(), TEXT("MULTIHOMEHTTP="), Address))
	{
	} else if (FParse::Value(FCommandLine::Get(), TEXT("MULTIHOME="), Address))
	{
	} else
	{
		Address = TEXT("127.0.0.1");
	}

	// use primary port
	FStringFormatOrderedArguments Args{ Address, PortsToUse[0]};
	FString ListFormat = FString::Format(*ResponseStr, Args);
	auto Response = FHttpServerResponse::Create(ListFormat, TEXT("text/html"));
	Response->Headers.Add(TEXT("Access-Control-Allow-Origin"), { TEXT("*") });
	OnComplete(MoveTemp(Response));
#endif
	return true;
}

bool UExternalRpcRegistry::HttpListOASv3JSONRoutes(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
#if WITH_RPC_REGISTRY
	FString ResponseStr;
	TArray<FName> OutRouteKeys;
	RegisteredRoutes.GetKeys(OutRouteKeys);
	TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&ResponseStr);


	JsonWriter->WriteObjectStart();

	// Based on OpenApi Spec v3.0.0, update this string
	// as necessary.
	JsonWriter->WriteValue(TEXT("openapi"), TEXT("3.0.0"));

	JsonWriter->WriteObjectStart(TEXT("info"));
	JsonWriter->WriteValue(TEXT("title"), FString::Printf(TEXT("UE-%s - RPC API"), FApp::GetProjectName()));
	JsonWriter->WriteValue(TEXT("description"), TEXT("Auto-generated Swagger API"));
	JsonWriter->WriteValue(TEXT("version"), FApp::GetBuildVersion());
	JsonWriter->WriteObjectEnd();

	JsonWriter->WriteArrayStart(TEXT("servers"));

	
	TArray<FString> Addresses;

	FString MultiHomeFromCommandLine;
	if (FParse::Value(FCommandLine::Get(), TEXT("MULTIHOMEHTTP="), MultiHomeFromCommandLine))
	{
		Addresses.Push(MultiHomeFromCommandLine);
	} else if (FParse::Value(FCommandLine::Get(), TEXT("MULTIHOME="), MultiHomeFromCommandLine))
	{
		Addresses.Push(MultiHomeFromCommandLine);
	}
	
	Addresses.Push(TEXT("127.0.0.1"));
	Addresses.Push(TEXT("localhost"));

	for (FString InternetAddr : Addresses)
	{
		JsonWriter->WriteObjectStart();
		// use primary port
		JsonWriter->WriteValue(TEXT("url"), FString::Printf(TEXT("http://%s:%d"), *InternetAddr, PortsToUse[0]));
		JsonWriter->WriteValue(TEXT("description"), TEXT("Default server access ip"));
		JsonWriter->WriteObjectEnd();
	}
	

	JsonWriter->WriteArrayEnd();

	JsonWriter->WriteObjectStart(TEXT("paths"));
	for (const FName& RouteKey : OutRouteKeys)
	{
		JsonWriter->WriteObjectStart(RegisteredRoutes[RouteKey].Handle->Path);
		JsonWriter->WriteObjectStart(GetHttpRouteVerbString(RegisteredRoutes[RouteKey].Handle->Verbs).ToLower());
		JsonWriter->WriteValue(TEXT("summary"), RouteKey.ToString());
		JsonWriter->WriteValue(TEXT("operationId"), RouteKey.ToString());

		if (!RegisteredRoutes[RouteKey].RpcCategory.IsEmpty())
		{
			JsonWriter->WriteArrayStart(TEXT("tags"));
			JsonWriter->WriteValue(RegisteredRoutes[RouteKey].RpcCategory);
			JsonWriter->WriteArrayEnd();
		}

		// TODO: Ask C++ implementers to provide a description of their RPC call should do.
		// We'll dump the InputContentType for now.
		if (!RegisteredRoutes[RouteKey].InputContentType.IsEmpty())
		{
			JsonWriter->WriteValue(TEXT("description"), RegisteredRoutes[RouteKey].InputContentType);
		}
		else
		{
			JsonWriter->WriteValue(TEXT("description"), TEXT("No content type required to call this."));
		}

		if (!RegisteredRoutes[RouteKey].ExpectedArguments.IsEmpty())
		{
			JsonWriter->WriteObjectStart(TEXT("requestBody"));
			JsonWriter->WriteObjectStart(TEXT("content"));
			JsonWriter->WriteObjectStart(TEXT("application/json"));

			JsonWriter->WriteObjectStart(TEXT("schema"));
			JsonWriter->WriteValue(TEXT("type"), TEXT("object"));

			JsonWriter->WriteObjectStart(TEXT("properties"));
			TArray<FString> RequiredObjects;
			for (const FExternalRpcArgumentDesc& ArgDesc : RegisteredRoutes[RouteKey].ExpectedArguments)
			{
				JsonWriter->WriteObjectStart(ArgDesc.Name);

				//TODO: Provide better typing in RPC framework so we can auto-gen some values here.
				JsonWriter->WriteValue(TEXT("description"), ArgDesc.Desc);
				//JsonWriter->WriteValue(TEXT("type"), ArgDesc.Type);
				JsonWriter->WriteValue(TEXT("type"), TEXT("string"));

				if (!ArgDesc.bIsOptional)
				{
					RequiredObjects.Push(ArgDesc.Name);
				}

				JsonWriter->WriteObjectEnd();
			}
			JsonWriter->WriteObjectEnd();

			if (RequiredObjects.Num() > 0)
			{
				JsonWriter->WriteArrayStart("required");
				for (FString RequiredName : RequiredObjects)
				{
					JsonWriter->WriteValue(RequiredName);
				}
				JsonWriter->WriteArrayEnd();
			}
			JsonWriter->WriteObjectEnd();

			JsonWriter->WriteObjectEnd();
			JsonWriter->WriteObjectEnd();
			JsonWriter->WriteObjectEnd();
		}

		JsonWriter->WriteObjectStart(TEXT("responses"));
		JsonWriter->WriteObjectStart(TEXT("200"));
		JsonWriter->WriteValue(TEXT("description"), TEXT("Successful return."));
		JsonWriter->WriteObjectEnd();
		JsonWriter->WriteObjectEnd();

		JsonWriter->WriteObjectEnd();
		JsonWriter->WriteObjectEnd();
	}
	JsonWriter->WriteObjectEnd();
	JsonWriter->WriteObjectEnd();
	JsonWriter->Close();
	auto Response = FHttpServerResponse::Create(ResponseStr, TEXT("application/json"));
	Response->Headers.Add(TEXT("Access-Control-Allow-Origin"), { TEXT("*") });
	OnComplete(MoveTemp(Response));
#endif
	return true;
}

IMPLEMENT_MODULE(FDefaultModuleImpl, ExternalRpcRegistry);

