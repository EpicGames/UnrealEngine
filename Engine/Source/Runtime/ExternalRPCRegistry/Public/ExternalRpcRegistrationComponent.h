// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HttpRequestHandler.h"
#include "Serialization/JsonWriter.h"
#include "ExternalRpcRegistry.h"


#include "ExternalRpcRegistrationComponent.generated.h"

#ifndef WITH_RPC_REGISTRY
#define WITH_RPC_REGISTRY (USE_RPC_REGISTRY_IN_SHIPPING || !UE_BUILD_SHIPPING )
#endif

#define UE_API EXTERNALRPCREGISTRY_API
class FJsonObject;
enum class EHttpServerRequestVerbs : uint16;
struct FExternalRouteInfo;
struct FExternalRpcArgumentDesc;
struct FHttpPath;
struct FHttpServerRequest;
struct FHttpServerResponse;
struct FKey;
UCLASS(MinimalAPI)
class UExternalRpcRegistrationComponent : public UObject
{

	GENERATED_BODY()
public:
	TArray<FName> RegisteredRoutes;
	FString SecuritySecret;


	// Outgoing RPC info
	FString SenderID;
	FString ListenerAddress;

	UE_API UExternalRpcRegistrationComponent();

	/**
	 * Deserializes the incoming request body into a JSON object
	 * 
	 * @param  InRequestBody  Incoming array of bytes from the request
	 * 
	 * @return Valid JSON object on a successful deserialization; otherwise returns a nullptr
	 */
	UE_API TSharedPtr<FJsonObject> GetJsonObjectFromRequestBody(const TArray<uint8>& InRequestBody);

	/**
	* Deregisters and cleans up all routes managed by this component.
	 */
	UE_API virtual void DeregisterHttpCallbacks();	
	
	/**
	 * Registration function for RPCs that should always be enabled as long as this Registration Component is active.
	 */
	UE_API virtual void RegisterAlwaysOnHttpCallbacks();

	/**
	 * Function called when RPCs are registered or deregistered to notify any listeners that they need to refresh their RPC list.
	 */
	UE_API void BroadcastRpcListChanged();
	/**
	 * Hooks up a given delegate to the assigned route for external access. Use if you already have an FExternalRouteInfo object.
	 *
	 * @param  InRouteInfo A fully initialized FExternalRouteInfo object that describes the overall endpoint.
	 * @param  Handler  The function delegate to call when this route is accessed.
	 * @param  bOverrideIfBound  Whether or not this registry should override any previously registered route.
	 */
	UE_API void RegisterHttpCallback(FExternalRouteInfo InRouteInfo, const FHttpRequestHandler& Handler, bool bOverrideIfBound = false);
	/**
	 * Hooks up a given delegate to the assigned route for external access. Handles creation of FExternalRouteInfo from its core parts.
	 *
	 * @param  RouteName   The friendly name of the route - used frequently in gauntlet for easier use.
	 * @param  HttpPath  Desired route for this delegate. Will be appended onto http://<host>:<listenport>
	 * @param  RequestVerbs  Verbs that should be accepted on this route.
	 * @param  Handler  The function delegate to call when this route is accessed.
	 * @param  bOverrideIfBound  Whether or not this registry should override any previously registered delegate at this route.
	 * @param  OptionalCategory  An optional category that can be added for organizational purposes
	 * @param  OptionalContentType  The sort of content type that is expected for the request body
	 * @param  OptionalInArguments  A list of arguments expected by this route. Can be set to optional or mandatory. 
	 */
	UE_API void RegisterHttpCallback(FName RouteName, const FHttpPath& HttpPath, const EHttpServerRequestVerbs& RequestVerbs, const FHttpRequestHandler& Handler, bool bOverrideIfBound = false, FString OptionalCategory = TEXT(""), FString OptionalContentType = {}, TArray<FExternalRpcArgumentDesc> OptionalInArguments = TArray<FExternalRpcArgumentDesc>());


	/**
	 * Send a message to all listeners with a custom category and payload for consumption by external processes.
	 *
	 * @param  MessageCategory   The category of the message to be sent.
	 * @param  MessagePayload  The payload of the message to be sent.
	 *
	 * @return Always needs to return true because of how the HttpRequest works.
	 */
	UE_API bool HttpSendMessageToListener(FString MessageCategory, FString MessagePayload);
	/**
	 * Send a message to all listeners describing the game IP of the current process.
	 * Mostly used when dealing with devices that have more than one IP to signify what its hostname should be.
	 *
	 * @param  TargetName   The name of the target process updating its ip.
	 * @param  MessagePayload  The IP of the process.
	 * 
	 * @return Always needs to return true because of how the HttpRequest works.
	 */
	UE_API bool HttpUpdateIpOnListener(FString TargetName, FString MessagePayload);
	/**
	 * Creates an Http Request Handler that gets routed through the additional handling of the ExternalRpcRegistry
	 * @param  InFunc   A delegate to be prepared for attachment to a route.
	 * @return An FHttpRequestHandler delegate object that can be assigned a route in RegisterHttpCallback
	 */
	UE_API FHttpRequestHandler CreateRouteHandle(TDelegate<bool(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)>	 InFunc);

	/**
	 * Generates a simple Json response object with success and reason for use when returning a response.
	 *
	 * @param  bInWasSuccessful   Whether or not the Rpc completed successfully
	 * @param  InValue  Optional result description string that will be returned with the success bool
	 * @param  bInFatal  Whether or not the response should be considered fatal and not worth retrying.
	 */
	UE_API TUniquePtr<FHttpServerResponse> CreateSimpleResponse(bool bInWasSuccessful, FString InValue = "", bool bInFatal = false);

protected:

	/**
	 * Appends timing context information to a response 
	 * 
	 * @param  InResponse		The response to append execution time to
	 * @param  InTimingContext  Handle to the timing context that was used to measure the execution time
	 */
	UE_API void AppendTimingContextToResponse(TUniquePtr<FHttpServerResponse>& InResponse, FRpcTimingContext& InTimingContext);
};

#undef UE_API
