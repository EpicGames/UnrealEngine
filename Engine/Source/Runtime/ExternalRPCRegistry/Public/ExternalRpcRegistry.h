// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HttpPath.h"
#include "HttpRequestHandler.h"
#include "HttpResultCallback.h"
#include "HttpRouteHandle.h"
#include "HttpServerRequest.h"
#include "IHttpRouter.h"
#include "Logging/LogMacros.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ExternalRpcRegistry.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogExternalRpcRegistry, Log, All);

USTRUCT()
struct FExternalRpcArgumentDesc
{
	GENERATED_BODY()
public:
	FString Name;
	FString Type;
	FString Desc;
	bool bIsOptional = false;

	FExternalRpcArgumentDesc() = default;
	FExternalRpcArgumentDesc(FString InName, FString InType, FString InDesc, bool bInIsOptional = false)
	{
		Name = InName;
		Type = InType;
		Desc = InDesc;
		bIsOptional = bInIsOptional;
	}

	bool operator==(const FExternalRpcArgumentDesc& Other) const
	{
		return
			Name == Other.Name &&
			Type == Other.Type &&
			Desc == Other.Desc &&
			bIsOptional == Other.bIsOptional;
	}

	bool operator!=(const FExternalRpcArgumentDesc& Other) const
	{
		return !(*this == Other);
	}
};

USTRUCT()
struct FExternalRouteInfo
{
	GENERATED_BODY()
public:
	FName RouteName;
	FHttpPath RoutePath;
	EHttpServerRequestVerbs RequestVerbs = EHttpServerRequestVerbs::VERB_NONE;
	FString InputContentType;
	TArray<FExternalRpcArgumentDesc> ExpectedArguments;
	FString RpcCategory = "Unknown";
	bool bAlwaysOn = false;

	FExternalRouteInfo() = default;

	FExternalRouteInfo(FName InRouteName, FHttpPath InRoutePath, EHttpServerRequestVerbs InRequestVerbs, FString InCategory = TEXT("Unknown"), bool bInAlwaysOn = false, FString InContentType = {}, TArray<FExternalRpcArgumentDesc> InArguments = TArray<FExternalRpcArgumentDesc>())
	{
		RouteName = InRouteName;
		RoutePath = InRoutePath;
		RequestVerbs = InRequestVerbs;
		InputContentType = InContentType;
		ExpectedArguments = InArguments;
		RpcCategory = InCategory;
		bAlwaysOn = false;
	}

	bool operator==(const FExternalRouteInfo& Other) const
	{
		return
			RouteName == Other.RouteName &&
			RoutePath == Other.RoutePath &&
			RequestVerbs == Other.RequestVerbs &&
			InputContentType == Other.InputContentType &&
			ExpectedArguments == Other.ExpectedArguments &&
			RpcCategory == Other.RpcCategory &&
			bAlwaysOn == Other.bAlwaysOn;
	}

	bool operator!=(const FExternalRouteInfo& Other) const
	{
		return !(*this == Other);
	}
};

USTRUCT()
struct FExternalRouteDesc
{
	GENERATED_BODY()
public:
	FHttpRouteHandle Handle;
	FString InputContentType;
	FString RpcCategory;
	TArray<FExternalRpcArgumentDesc> ExpectedArguments;
	FExternalRouteDesc() = default;
	FExternalRouteDesc(FHttpRouteHandle InHandle, FString InContentType, TArray<FExternalRpcArgumentDesc> InArguments, FString InCategory = "")
	{
		Handle = InHandle;
		InputContentType = InContentType;
		ExpectedArguments = InArguments;
		RpcCategory = InCategory;
	}
};

USTRUCT()
struct FRpcLedgerEntry
{
	GENERATED_BODY()
public:
	FString RpcName;
	FString RequestBody;
	FDateTime RequestTime;
	FRpcLedgerEntry() = default;
	FRpcLedgerEntry(FString InName, FString InBody)
	{
		RpcName = InName;
		RequestBody = InBody;
		RequestTime = FDateTime::UtcNow();
	}
};


USTRUCT()
struct FRpcConfig
{
	GENERATED_BODY()
	FRpcConfig() = default;

	TMap<FString, int> PortMapping;

	// it is valid to have mappings, but no active ports, in the case of a client with multiple endpoint hosts
	TArray<int> ActiveHostPorts;
};

/**
 * Context struct used to measure execution time of RPCs
 * Execution time can be returned in the response to measure actual runtime of the RPC minus the network traffic
*/
USTRUCT()
struct FRpcTimingContext
{
	GENERATED_BODY()
public:

	double ExecutionTimeMs;

	FRpcTimingContext() 
		: ExecutionTimeMs(-1.0)
	{ }

	void StartTiming()
	{
		StartTime = FDateTime::Now();
	}

	double StopTiming()
	{
		ExecutionTimeMs = (FDateTime::Now() - StartTime).GetTotalMilliseconds();
		return ExecutionTimeMs;
	}

private:
	FDateTime StartTime;
};


/**
 * This class is designed to be a singleton that handles registry, maintenance, and cleanup of any REST endpoints exposed on the process 
 * for use in communicating with the process externally. 
 */
UCLASS(MinimalAPI)
class UExternalRpcRegistry : public UObject
{
	GENERATED_BODY()
protected:
	static EXTERNALRPCREGISTRY_API UExternalRpcRegistry * ObjectInstance;
	TMap<FName, FExternalRouteDesc> RegisteredRoutes;
	TArray<FString> ActiveRpcCategories;
	TArray<FRpcLedgerEntry> RequestLedger;
	TMap<FName, FName> MultiPortRouteMapping;
	TArray<int> PortsToUse;

public:
	static EXTERNALRPCREGISTRY_API UExternalRpcRegistry * GetInstance();
	static EXTERNALRPCREGISTRY_API bool IsEnabled();
	static EXTERNALRPCREGISTRY_API const FRpcConfig& GetRpcConfig();
	EXTERNALRPCREGISTRY_API ~UExternalRpcRegistry();

	int RequestLedgerCapacity = 10;

	int GetPortToUse()
	{
		return PortsToUse.Num() ? PortsToUse[0] : 0;
	}

	/**
	* Check if this Rpc is from a category that is meant to be enabled.
	*/
	EXTERNALRPCREGISTRY_API bool IsActiveRpcCategory(FString InCategory);

	/**
	 * Try to get a route registered under given friendly name. Returns false if could not be found.
	 */
	EXTERNALRPCREGISTRY_API bool GetRegisteredRoute(FName RouteName, FExternalRouteInfo& OutRouteInfo);

	EXTERNALRPCREGISTRY_API void RegisterNewRoute(FExternalRouteInfo InRouteInfo, const FHttpRequestHandler& Handler, bool bOverrideIfBound = false);

	/**
	 * Register a new route.
	 * Will override existing routes if option is set, otherwise will error and fail to bind.
	 */
	EXTERNALRPCREGISTRY_API void RegisterNewRouteWithArguments(FName RouteName, const FHttpPath& HttpPath, const EHttpServerRequestVerbs& RequestVerbs, const FHttpRequestHandler& Handler, TArray<FExternalRpcArgumentDesc> InArguments, bool bOverrideIfBound = false, bool bIsAlwaysOn = false, FString OptionalCategory = TEXT("Unknown"), FString OptionalContentType = {});

	/**
	* Deprecated way to register a new route.
	* Will override existing routes if option is set, otherwise will error and fail to bind.
	 */
	EXTERNALRPCREGISTRY_API void RegisterNewRoute(FName RouteName, const FHttpPath& HttpPath, const EHttpServerRequestVerbs& RequestVerbs, const FHttpRequestHandler& Handler, bool bOverrideIfBound = false, bool bIsAlwaysOn = false, FString OptionalCategory = TEXT("Unknown"), FString OptionalContentType = {}, FString OptionalExpectedFormat = {});

	/**
	 * Clean up all routes - generally called as part of the destructor to make sure we don't have any function pointers dangling around.
	 */
	EXTERNALRPCREGISTRY_API void CleanUpAllRoutes();

	/**
	 * Clean up a route.
	 * Can be set to fail if trying to unbind an unbound route.
	 */
	EXTERNALRPCREGISTRY_API void CleanUpRoute(FName RouteName, bool bFailIfUnbound = false);

	/**
	 * Default Route Listing http call. Spits out all registered routes and describes them via a REST API call.
	 * Always registered at /listrpcs GET by default
	 */
	EXTERNALRPCREGISTRY_API bool HttpListOpenRoutes(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	/**
	* Records an RPC call into the ledger, plus trims the ledger if it is too large.
	 */
	EXTERNALRPCREGISTRY_API void AddRequestToLedger(const FHttpServerRequest& Request);

	/**
	 * Default request history call. Lists all calls recorded in the ledger, showing RPC name, request time, request body
	 * Always registered at /requesthistory GET by default
	 */
	EXTERNALRPCREGISTRY_API bool HttpPrintRequestLedger(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);

	/**
	 * Route Listing http, formatted for Swagger OASv3. Spits out all registered routes and describes them via a REST API call.
	 * Always registered at /swagger.json GET by default
	 */
	EXTERNALRPCREGISTRY_API bool HttpListOASv3JSONRoutes(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);


	/**
	 * Route Showing SwaggerUI using /swagger.json to describe the current REST API.
	 * Always registered at /swagger/html.index GET by default
	 */
	EXTERNALRPCREGISTRY_API bool HttpSwaggerUI(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);

};
